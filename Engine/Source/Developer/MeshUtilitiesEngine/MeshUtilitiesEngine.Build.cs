// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class MeshUtilitiesEngine : ModuleRules
{
	public MeshUtilitiesEngine(ReadOnlyTargetRules Target) : base(Target)
	{
        PublicDependencyModuleNames.AddRange(
            new string[] {
				"Engine",
				"MeshUtilitiesCommon",
			}
        );

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
            }
		);
	}
}
