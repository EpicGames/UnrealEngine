// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshUtilitiesEngine.h"

#include "MeshUtilitiesCommon.h"
#include "Engine/SkeletalMesh.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/SkinWeightVertexBuffer.h"

// Find the most dominant bone for each vertex
FBoneIndexType GetDominantBoneIndex(const FSkinWeightInfo& VertexInfluences)
{
	FBoneIndexType MaxWeightBone = 0;
	uint16 MaxWeightWeight = 0;
	for (int32 InfluenceIndex = 0; InfluenceIndex < MAX_TOTAL_INFLUENCES; InfluenceIndex++)
	{
		if (VertexInfluences.InfluenceWeights[InfluenceIndex] > MaxWeightWeight)
		{
			MaxWeightWeight = VertexInfluences.InfluenceWeights[InfluenceIndex];
			MaxWeightBone = VertexInfluences.InfluenceBones[InfluenceIndex];
		}
	}

	return MaxWeightBone;
}

void FMeshUtilitiesEngine::CalcBoneVertInfos(USkeletalMesh* SkeletalMesh, TArray<FBoneVertInfo>& Infos, bool bOnlyDominant, int32 SourceLodIndex)
{
	//Find the source LOD index
	const int32 LodIndex = SkeletalMesh->IsValidLODIndex(SourceLodIndex) ? SourceLodIndex : 0;

	SkeletalMesh->CalculateInvRefMatrices();
	check(SkeletalMesh->GetRefSkeleton().GetRawBoneNum() == SkeletalMesh->GetRefBasesInvMatrix().Num());

	Infos.Empty();
	
	const FSkeletalMeshRenderData* RenderData = SkeletalMesh->GetResourceForRendering();
	if (!RenderData || !RenderData->LODRenderData.IsValidIndex(LodIndex))
	{
		return;
	}

	const FSkeletalMeshLODRenderData& LODRenderData = RenderData->LODRenderData[LodIndex];
	if (!LODRenderData.StaticVertexBuffers.PositionVertexBuffer.GetAllowCPUAccess() ||
		!LODRenderData.StaticVertexBuffers.StaticMeshVertexBuffer.GetAllowCPUAccess() ||
		!LODRenderData.SkinWeightVertexBuffer.GetNeedsCPUAccess())
	{
		return;
	}

	const FPositionVertexBuffer& PositionBuffer = LODRenderData.StaticVertexBuffers.PositionVertexBuffer;
	const FStaticMeshVertexBuffer& TangentBuffer = LODRenderData.StaticVertexBuffers.StaticMeshVertexBuffer;

	Infos.AddZeroed(SkeletalMesh->GetRefSkeleton().GetRawBoneNum());

	TArray<FSkinWeightInfo> SkinWeights;
	LODRenderData.SkinWeightVertexBuffer.GetSkinWeights(SkinWeights);

	for (int32 SectionIndex = 0; SectionIndex < LODRenderData.RenderSections.Num(); SectionIndex++)
	{
		const FSkelMeshRenderSection& RenderSection =  LODRenderData.RenderSections[SectionIndex];
		for (int32 SectionVertexIndex = 0; SectionVertexIndex < static_cast<int32>(RenderSection.NumVertices); SectionVertexIndex++)
		{
			const int32 VertexIndex = SectionVertexIndex + RenderSection.BaseVertexIndex;
			if (bOnlyDominant)
			{
				const int32 BoneIndex = RenderSection.BoneMap[GetDominantBoneIndex(SkinWeights[VertexIndex])];

				FVector3f LocalPos = SkeletalMesh->GetRefBasesInvMatrix()[BoneIndex].TransformPosition(PositionBuffer.VertexPosition(VertexIndex));
				Infos[BoneIndex].Positions.Add(LocalPos);

				FVector3f LocalNormal = SkeletalMesh->GetRefBasesInvMatrix()[BoneIndex].TransformVector(TangentBuffer.VertexTangentZ(VertexIndex));
				Infos[BoneIndex].Normals.Add(LocalNormal);
			}
			else
			{
				const FSkinWeightInfo& BoneWeightInfo = SkinWeights[VertexIndex];
				for (int32 InfluenceIndex = 0; InfluenceIndex < MAX_TOTAL_INFLUENCES; InfluenceIndex++)
				{
					if (BoneWeightInfo.InfluenceWeights[InfluenceIndex] > 0)
					{
						int32 BoneIndex = RenderSection.BoneMap[BoneWeightInfo.InfluenceBones[InfluenceIndex]];
					
						FVector3f LocalPos = SkeletalMesh->GetRefBasesInvMatrix()[BoneIndex].TransformPosition(PositionBuffer.VertexPosition(VertexIndex));
						Infos[BoneIndex].Positions.Add(LocalPos);

						FVector3f LocalNormal = SkeletalMesh->GetRefBasesInvMatrix()[BoneIndex].TransformVector(TangentBuffer.VertexTangentZ(VertexIndex));
						Infos[BoneIndex].Normals.Add(LocalNormal);
					}
				}
			}
		}
	}
}
