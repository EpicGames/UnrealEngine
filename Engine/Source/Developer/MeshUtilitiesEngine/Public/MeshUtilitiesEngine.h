// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API MESHUTILITIESENGINE_API

class USkeletalMesh;
struct FBoneVertInfo;

class FMeshUtilitiesEngine
{
public:
	/**
	 *	Calculate the verts associated weighted to each bone of the skeleton.
	 *	The vertices returned are in the local space of the bone.
	 *
	 *	@param	SkeletalMesh	The target skeletal mesh.
	 *	@param	Infos			The output array of vertices associated with each bone.
	 *	@param	bOnlyDominant	Controls whether a vertex is added to the info for a bone if it is most controlled by that bone, or if that bone has ANY influence on that vert.
	 *	@param	SourceLodIndex	Use the specified LOD index to query the bone vert info.
	 */
	static UE_API void CalcBoneVertInfos(USkeletalMesh* SkeletalMesh, TArray<FBoneVertInfo>& Infos, bool bOnlyDominant, int32 SourceLodIndex);
};

#undef UE_API
